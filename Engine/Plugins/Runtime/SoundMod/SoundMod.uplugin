{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Sound Module Plugin",
	"Description" : "Supports playback of ProTracker (MOD),  Scream Tracker 3 (S3M), Fast Tracker II (XM), and Impulse Tracker (IT) files.",
	"Category" : "Audio",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "SoundMod",
			"Type" : "Runtime",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Win64",
				"Mac",
				"Linux",
				"Android"
			]
		},
		{
			"Name" : "SoundModImporter",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}