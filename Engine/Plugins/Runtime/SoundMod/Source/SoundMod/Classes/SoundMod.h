// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** 
 * A sound module file
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Serialization/BulkData.h"
#include "Sound/SoundBase.h"
#include "SoundMod.generated.h"

struct FActiveSound;
struct FSoundParseParameters;

UCLASS(hidecategories=Object, MinimalAPI, BlueprintType)
class USoundMod : public USoundBase
{
	GENERATED_UCLASS_BODY()

	/** If set, when played directly (not through a sound cue) the nid will be played looping. */
	UPROPERTY(EditAnywhere, Category=SoundMod)
	uint32 bLooping:1;

	/** The mod file data */
	FByteBulkData				RawData;

private:

	/** Memory containing the data copied from the compressed bulk data */
	uint8*	ResourceData;

public:	
	//~ Begin UObject Interface. 
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface. 

	//~ Begin USoundBase Interface.
	virtual bool IsPlayable() const override;
	virtual void Parse(class FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances) override;
	//~ End USoundBase Interface.
};

