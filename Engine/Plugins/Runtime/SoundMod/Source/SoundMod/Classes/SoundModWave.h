// Copyright Epic Games, Inc. All Rights Reserved.

/**
* Playable sound object for wave files that are generated from a Sound Mod
*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "xmp.h"
#include "Sound/SoundWaveProcedural.h"
#include "SoundModWave.generated.h"

UCLASS()
class USoundModWave : public USoundWaveProcedural
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<class USoundMod> SoundMod;

	xmp_context xmpContext;

	//~ Begin USoundWave Interface.
	virtual int32 GeneratePCMData(uint8* PCMData, const int32 SamplesNeeded) override;
	//~ End USoundWave Interface.

};
