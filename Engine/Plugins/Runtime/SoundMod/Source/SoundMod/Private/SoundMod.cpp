// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundMod.h"
#include "SoundModWave.h"
#include "ActiveSound.h"

USoundMod::USoundMod(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USoundMod::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	RawData.Serialize(Ar, this);
}

void USoundMod::Parse(class FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances)
{
	FWaveInstance* WaveInstance = ActiveSound.FindWaveInstance(NodeWaveInstanceHash);

	// Create a new WaveInstance if this SoundWave doesn't already have one associated with it.
	if (!WaveInstance)
	{
		const int32 SampleRate = 44100;

		// Create a new wave instance and associate with the ActiveSound
		WaveInstance = &ActiveSound.AddWaveInstance(NodeWaveInstanceHash);

		// Create streaming wave object
		USoundModWave* ModWave = NewObject<USoundModWave>();
		ModWave->SetSampleRate(SampleRate);
		ModWave->NumChannels = 2;
		ModWave->Duration = INDEFINITELY_LOOPING_DURATION;
		ModWave->bLooping = bLooping;

		if (!ResourceData)
		{
			RawData.GetCopy((void**)&ResourceData, true);
		}

		ModWave->xmpContext = xmp_create_context();
		xmp_load_module_from_memory(ModWave->xmpContext, ResourceData, RawData.GetBulkDataSize());
		xmp_start_player(ModWave->xmpContext, SampleRate, 0);

		WaveInstance->WaveData = ModWave;
	}

	WaveInstance->WaveData->Parse(AudioDevice, NodeWaveInstanceHash, ActiveSound, ParseParams, WaveInstances);
}

bool USoundMod::IsPlayable() const
{
	return true;
}
