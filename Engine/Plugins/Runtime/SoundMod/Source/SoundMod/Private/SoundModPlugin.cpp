// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundModPlugin.h"

class FSoundModPlugin : public ISoundModPlugin
{
public:
	// IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End of IModuleInterface implementation
};

IMPLEMENT_MODULE( FSoundModPlugin, SoundMod )



void FSoundModPlugin::StartupModule()
{
	// This code will execute after your module is loaded into memory (but after global variables are initialized, of course.)
}


void FSoundModPlugin::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}



