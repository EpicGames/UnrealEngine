// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundModWave.h"

USoundModWave::USoundModWave(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, xmpContext(NULL)
{
	bCanProcessAsync = true;
}

DECLARE_CYCLE_STAT(TEXT("Sound Mod Generate Data"), STAT_SoundModGeneratePCMData, STATGROUP_Audio );

int32 USoundModWave::GeneratePCMData(uint8* PCMData, const int32 SamplesNeeded)
{
	SCOPE_CYCLE_COUNTER(STAT_SoundModGeneratePCMData);

	int32 BytesGenerated = Super::GeneratePCMData(PCMData, SamplesNeeded);
	int32 BytesDesired = (SamplesNeeded * sizeof(int16)) - BytesGenerated;

	if (BytesDesired > 0)
	{
		while (xmp_play_frame(xmpContext) == 0)
		{
			xmp_frame_info xmpFrameInfo;
			xmp_get_frame_info(xmpContext, &xmpFrameInfo);

			const int32 BytesToCopy = FMath::Min<int32>(xmpFrameInfo.buffer_size, BytesDesired);

			if (BytesToCopy > 0)
			{
				FMemory::Memcpy((void*)(PCMData + BytesGenerated), xmpFrameInfo.buffer, BytesToCopy);

				BytesDesired -= BytesToCopy;
				BytesGenerated += BytesToCopy;

				if (BytesDesired == 0)
				{
					if (BytesToCopy != xmpFrameInfo.buffer_size)
					{
						QueueAudio((uint8*)xmpFrameInfo.buffer + BytesToCopy, xmpFrameInfo.buffer_size - BytesToCopy);
					}
					break;
				}
			}
		}
	}

	return BytesGenerated;
}
