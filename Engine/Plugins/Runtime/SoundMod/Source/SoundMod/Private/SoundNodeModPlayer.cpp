// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundNodeModPlayer.h"
#include "SoundMod.h"
#include "ActiveSound.h"
#include "UObject/FrameworkObjectVersion.h"

#define LOCTEXT_NAMESPACE "SoundNodeModPlayer"

void USoundNodeModPlayer::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::HardSoundReferences)
	{
		if (Ar.IsLoading())
		{
			Ar << SoundMod;
		}
		else if (Ar.IsSaving())
		{
			USoundMod* HardReference = (ShouldHardReferenceAsset(Ar.CookingTarget()) ? SoundMod.Get() : nullptr);
			Ar << HardReference;
		}
	}
}

void USoundNodeModPlayer::LoadAsset(bool bAddToRoot)
{
	if (IsAsyncLoading())
	{
		SoundMod = SoundModAssetPtr.Get();
		if (SoundMod == nullptr)
		{
			const FString LongPackageName = SoundModAssetPtr.GetLongPackageName();
			if (!LongPackageName.IsEmpty())
			{
				bAsyncLoading = true;
				LoadPackageAsync(LongPackageName, FLoadPackageAsyncDelegate::CreateUObject(this, &USoundNodeModPlayer::OnSoundModLoaded, bAddToRoot));
			}
		}
		else if (bAddToRoot)
		{
			SoundMod->AddToRoot();
		}
	}
	else
	{
		SoundMod = SoundModAssetPtr.LoadSynchronous();
		if (bAddToRoot && SoundMod)
		{
			SoundMod->AddToRoot();
		}
	}
}

void USoundNodeModPlayer::ClearAssetReferences()
{
	SoundMod = nullptr;
}

void USoundNodeModPlayer::OnSoundModLoaded(const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result, bool bAddToRoot)
{
	if (Result == EAsyncLoadingResult::Succeeded)
	{
		SoundMod = SoundModAssetPtr.Get();
		if (bAddToRoot && SoundMod)
		{
			SoundMod->AddToRoot();
		}
	}
	bAsyncLoading = false;
}

void USoundNodeModPlayer::SetSoundMod(USoundMod* InSoundMod)
{
	SoundMod = InSoundMod;
	SoundModAssetPtr = InSoundMod;
}

#if WITH_EDITOR
void USoundNodeModPlayer::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(USoundNodeModPlayer, SoundModAssetPtr))
	{
		LoadAsset();
	}
}
#endif

void USoundNodeModPlayer::ParseNodes( FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances )
{
	if (bAsyncLoading)
	{
		SoundMod = SoundModAssetPtr.LoadSynchronous();
		bAsyncLoading = false;
	}
	if (SoundMod)
	{
		// The SoundWave's bLooping is only for if it is directly referenced, so clear it
		// in the case that it is being played from a player
		bool bModIsLooping = SoundMod->bLooping;
		SoundMod->bLooping = false;

		if (bLooping)
		{
			FSoundParseParameters UpdatedParams = ParseParams;
			UpdatedParams.bLooping = true;
			SoundMod->Parse(AudioDevice, NodeWaveInstanceHash, ActiveSound, UpdatedParams, WaveInstances);
		}
		else
		{
			SoundMod->Parse(AudioDevice, NodeWaveInstanceHash, ActiveSound, ParseParams, WaveInstances);
		}

		SoundMod->bLooping = bModIsLooping;
	}
}

float USoundNodeModPlayer::GetDuration()
{
	float Duration = 0.f;
	if (SoundMod)
	{
		if (bLooping)
		{
			Duration = INDEFINITELY_LOOPING_DURATION;
		}
		else
		{
			Duration = SoundMod->Duration;
		}
	}
	return Duration;
}

#if WITH_EDITOR
FText USoundNodeModPlayer::GetTitle() const
{
	FText SoundModName;
	if (SoundMod)
	{
		SoundModName = FText::FromString(SoundMod->GetFName().ToString());
	}
	else
	{
		SoundModName = LOCTEXT("NoSoundMod", "NONE");
	}

	FText Title;

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("Description"), Super::GetTitle());
	Arguments.Add(TEXT("SoundModName"), SoundModName);

	if (bLooping)
	{
		Title = FText::Format(LOCTEXT("LoopingSoundModDescription", "Looping {Description} : {SoundModName}"), Arguments);
	}
	else
	{
		Title = FText::Format(LOCTEXT("NonLoopingSoundModDescription", "{Description} : {SoundModName}"), Arguments);
	}

	return Title;
}
#endif

// A Mod Player is the end of the chain and has no children
int32 USoundNodeModPlayer::GetMaxChildNodes() const
{
	return 0;
}


#undef LOCTEXT_NAMESPACE
