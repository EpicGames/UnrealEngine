// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "Sound/SoundNodeAssetReferencer.h"
#include "SoundNodeModPlayer.generated.h"

class FAudioDevice;
class USoundMod;
struct FActiveSound;
struct FPropertyChangedEvent;
struct FSoundParseParameters;
struct FWaveInstance;

/** 
 * Sound node that contains a reference to the mod file to be played
 */
UCLASS(hidecategories=Object, editinlinenew, MinimalAPI, meta=( DisplayName="Mod Player" ))
class USoundNodeModPlayer : public USoundNodeAssetReferencer
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Category=ModPlayer, meta=(DisplayName="Sound Mod"))
	TSoftObjectPtr<USoundMod> SoundModAssetPtr;

	UPROPERTY(transient)
	TObjectPtr<USoundMod> SoundMod;

	UPROPERTY(EditAnywhere, Category=ModPlayer)
	uint32 bLooping:1;

	uint32 bAsyncLoading:1;

	void OnSoundModLoaded(const FName& PackageName, UPackage * Package, EAsyncLoadingResult::Type Result, bool bAddToRoot);

public:	

	USoundMod* GetSoundMod() const { return SoundMod; }
	void SetSoundMod(USoundMod* SoundMod);

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface

	//~ Begin USoundNode Interface
	virtual int32 GetMaxChildNodes() const override;
	virtual float GetDuration() override;
	virtual void ParseNodes(FAudioDevice* AudioDevice, const UPTRINT NodeWaveInstanceHash, FActiveSound& ActiveSound, const FSoundParseParameters& ParseParams, TArray<FWaveInstance*>& WaveInstances) override;
#if WITH_EDITOR
	virtual FText GetTitle() const override;
#endif
	//~ End USoundNode Interface

	//~ Begin USoundNodeAssetReferencer Interface
	virtual void LoadAsset(bool bAddToRoot = false) override;
	virtual void ClearAssetReferences() override;
	//~ End USoundNode Interface

};

