// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class SoundMod : ModuleRules
    {
        public SoundMod(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
                new string[]
				{
					"AudioExtensions",
					"Core",
					"CoreUObject",
                    "Engine",
				}
            );

			PublicDefinitions.AddRange(
				new string[] {
					"BUILDING_STATIC"
				}
			);

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				// VS2015 updated some of the CRT definitions but not all of the Windows SDK has been updated to match.
				// Microsoft provides this shim library to enable building with VS2015 until they fix everything up.
				//@todo: remove when no longer neeeded (no other code changes should be necessary).
				if (Target.WindowsPlatform.bNeedsLegacyStdioDefinitionsLib)
				{
					PublicSystemLibraries.Add("legacy_stdio_definitions.lib");
				}
			}

			// Link with managed Perforce wrapper assemblies
			AddEngineThirdPartyPrivateStaticDependencies(Target, "coremod");
		
		}
    }
}
