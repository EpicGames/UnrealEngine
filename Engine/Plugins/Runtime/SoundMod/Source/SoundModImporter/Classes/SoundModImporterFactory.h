// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "SoundModImporterFactory.generated.h"

// Imports a sound module file
UCLASS()
class USoundModImporterFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	virtual UObject* FactoryCreateBinary(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn) override;
	// End of UFactory interface
};

