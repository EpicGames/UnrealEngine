// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions_SoundMod.h"
#include "Components/AudioComponent.h"
#include "SoundMod.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "SoundModImporter"

UClass* FAssetTypeActions_SoundMod::GetSupportedClass() const
{
	return USoundMod::StaticClass();
}

void FAssetTypeActions_SoundMod::GetActions(const TArray<UObject*>& InObjects, FToolMenuSection& Section)
{
	auto Sounds = GetTypedWeakObjectPtrs<USoundMod>(InObjects);

	Section.AddMenuEntry(
		"Sound_PlaySound",
		LOCTEXT("Sound_PlaySound", "Play"),
		LOCTEXT("Sound_PlaySoundTooltip", "Plays the selected sound."),
		FSlateIcon(),
		FUIAction(
		FExecuteAction::CreateSP(this, &FAssetTypeActions_SoundMod::ExecutePlaySound, Sounds),
		FCanExecuteAction::CreateSP(this, &FAssetTypeActions_SoundMod::CanExecutePlayCommand, Sounds)
		)
		);

	Section.AddMenuEntry(
		"Sound_StopSound",
		LOCTEXT("Sound_StopSound", "Stop"),
		LOCTEXT("Sound_StopSoundTooltip", "Stops the selected sounds."),
		FSlateIcon(),
		FUIAction(
		FExecuteAction::CreateSP(this, &FAssetTypeActions_SoundMod::ExecuteStopSound, Sounds),
		FCanExecuteAction()
		)
		);
}

bool FAssetTypeActions_SoundMod::CanExecutePlayCommand(TArray<TWeakObjectPtr<USoundMod>> Objects) const
{
	return Objects.Num() == 1;
}

bool FAssetTypeActions_SoundMod::AssetsActivatedOverride(const TArray<UObject*>& InObjects, EAssetTypeActivationMethod::Type ActivationType)
{
	if (ActivationType == EAssetTypeActivationMethod::Previewed)
	{
		USoundMod* TargetSound = NULL;

		for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
		{
			TargetSound = Cast<USoundMod>(*ObjIt);
			if (TargetSound)
			{
				// Only target the first valid sound cue
				break;
			}
		}

		UAudioComponent* PreviewComp = GEditor->GetPreviewAudioComponent();
		if (PreviewComp && PreviewComp->IsPlaying())
		{
			// Already previewing a sound, if it is the target cue then stop it, otherwise play the new one
			if (!TargetSound || PreviewComp->Sound == TargetSound)
			{
				StopSound();
			}
			else
			{
				PlaySound(TargetSound);
			}
		}
		else
		{
			// Not already playing, play the target sound cue if it exists
			PlaySound(TargetSound);
		}
		return true;
	}
	return false;
}

void FAssetTypeActions_SoundMod::ExecutePlaySound(TArray<TWeakObjectPtr<USoundMod>> Objects)
{
	for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		USoundMod* Sound = (*ObjIt).Get();
		if (Sound)
		{
			// Only play the first valid sound
			PlaySound(Sound);
			break;
		}
	}
}

void FAssetTypeActions_SoundMod::ExecuteStopSound(TArray<TWeakObjectPtr<USoundMod>> Objects)
{
	StopSound();
}

void FAssetTypeActions_SoundMod::PlaySound(USoundMod* Sound)
{
	if (Sound)
	{
		GEditor->PlayPreviewSound(Sound);
	}
	else
	{
		StopSound();
	}
}

void FAssetTypeActions_SoundMod::StopSound()
{
	GEditor->ResetPreviewAudioComponent();
}

#undef LOCTEXT_NAMESPACE
