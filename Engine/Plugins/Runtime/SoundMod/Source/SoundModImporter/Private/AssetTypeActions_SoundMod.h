// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions_Base.h"

class USoundMod;

class FAssetTypeActions_SoundMod: public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override{ return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SoundMod", "Sound Mod"); }
	virtual FColor GetTypeColor() const override{ return FColor(255, 175, 0); }
	virtual UClass* GetSupportedClass() const override;
	virtual void GetActions(const TArray<UObject*>& InObjects, struct FToolMenuSection& Section) override;
	virtual bool AssetsActivatedOverride(const TArray<UObject*>& InObjects, EAssetTypeActivationMethod::Type ActivationType) override;
	virtual uint32 GetCategories() override{ return EAssetTypeCategories::Sounds; }
	virtual bool CanFilter() override { return false; }

private:
	/** Handler for when PlaySound is selected */
	void ExecutePlaySound(TArray<TWeakObjectPtr<USoundMod>> Objects);

	/** Handler for when StopSound is selected */
	void ExecuteStopSound(TArray<TWeakObjectPtr<USoundMod>> Objects);

	/** Returns true if only one sound is selected to play */
	bool CanExecutePlayCommand(TArray<TWeakObjectPtr<USoundMod>> Objects) const;

	/** Plays the specified sound wave */
	void PlaySound(USoundMod* Sound);

	/** Stops any currently playing sounds */
	void StopSound();
};
