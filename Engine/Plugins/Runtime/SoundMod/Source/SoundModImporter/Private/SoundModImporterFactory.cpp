// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundModImporterFactory.h"
#include "AudioAnalytics.h"
#include "Editor.h"
#include "SoundMod.h"
#include "Subsystems/ImportSubsystem.h"
#include "xmp.h"
#include "Components/AudioComponent.h"

//////////////////////////////////////////////////////////////////////////
// USoundModImporterFactory

static bool bSoundModFactorySuppressImportOverwriteDialog = false;

USoundModImporterFactory::USoundModImporterFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	SupportedClass = USoundMod::StaticClass();

	bEditorImport = true;

	Formats.Add(TEXT("mod;Protracker file"));
	Formats.Add(TEXT("s3m;Scream Tracker 3 file"));
	Formats.Add(TEXT("xm;Fast Tracker II file"));
	Formats.Add(TEXT("it;Impulse Tracker file"));
}

UObject* USoundModImporterFactory::FactoryCreateBinary
(
	UClass*				Class,
	UObject*			InParent,
	FName				Name,
	EObjectFlags		Flags,
	UObject*			Context,
	const TCHAR*		FileType,
	const uint8*&		Buffer,
	const uint8*		BufferEnd,
	FFeedbackContext*	Warn
)
{
	// if the sound already exists, remember the user settings
	USoundMod* ExistingSound = FindObject<USoundMod>(InParent, *Name.ToString());

	// TODO - Audio Threading. This needs to be sent to the audio device and wait on stopping the sounds
	TArray<UAudioComponent*> ComponentsToRestart;
	FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();
	if (AudioDeviceManager && ExistingSound)
	{
		// TODO: Generalize the stop sounds function
		//AudioDeviceManager->StopSoundsForReimport(ExistingSound, ComponentsToRestart);
	}

	bool bUseExistingSettings = bSoundModFactorySuppressImportOverwriteDialog;

	if (ExistingSound && !bUseExistingSettings && !GIsAutomationTesting)
	{
		DisplayOverwriteOptionsDialog(FText::Format(
			NSLOCTEXT("SoundModImporterFactory", "ImportOverwriteWarning", "You are about to import '{0}' over an existing sound."),
			FText::FromName(Name)));

		switch (OverwriteYesOrNoToAllState)
		{
			case EAppReturnType::Yes:
			case EAppReturnType::YesAll:
			{
				// Overwrite existing settings
				bUseExistingSettings = false;
				break;
			}
			case EAppReturnType::No:
			case EAppReturnType::NoAll:
			{
				// Preserve existing settings
				bUseExistingSettings = true;
				break;
			}
			default:
			{
				GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, NULL);
				return NULL;
			}
		}
	}

	// Reset the flag back to false so subsequent imports are not suppressed unless the code explicitly suppresses it
	bSoundModFactorySuppressImportOverwriteDialog = false;

	TArray<uint8> RawModData;
	RawModData.Empty(BufferEnd - Buffer);
	RawModData.AddUninitialized(BufferEnd - Buffer);
	FMemory::Memcpy(RawModData.GetData(), Buffer, RawModData.Num());

	// TODO: Validate that this is actually a mod file
	xmp_context xmpContext = xmp_create_context();
	if (xmp_load_module_from_memory(xmpContext, RawModData.GetData(), RawModData.Num()) != 0)
	{
		return NULL;
	}
	xmp_module_info xmpModuleInfo;
	xmp_get_module_info(xmpContext, &xmpModuleInfo);

	// Use pre-existing sound if it exists and we want to keep settings,
	// otherwise create new sound and import raw data.
	USoundMod* Sound = (bUseExistingSettings && ExistingSound) ? ExistingSound : NewObject<USoundMod>(InParent, Name, Flags);

	Sound->Duration = xmpModuleInfo.seq_data->duration / 1000.f;

	xmp_release_module(xmpContext);
	xmp_free_context(xmpContext);

	Sound->RawData.Lock(LOCK_READ_WRITE);
	void* LockedData = Sound->RawData.Realloc(BufferEnd - Buffer);
	FMemory::Memcpy(LockedData, Buffer, BufferEnd - Buffer);
	Sound->RawData.Unlock();

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, Sound);

	for (int32 ComponentIndex = 0; ComponentIndex < ComponentsToRestart.Num(); ++ComponentIndex)
	{
		ComponentsToRestart[ComponentIndex]->Play();
	}

	Audio::Analytics::RecordEvent_Usage("SoundModule.BinaryCreated");

	return Sound;
}
