// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundModImporterPrivate.h"
#include "AssetTypeActions_SoundMod.h"

//////////////////////////////////////////////////////////////////////////
// FSoundModImporterModule

class FSoundModImporterModule : public FDefaultModuleImpl
{
public:
	/** Asset type actions for SoundMod assets.  Cached here so that we can unregister it during shutdown. */
	TSharedPtr<FAssetTypeActions_SoundMod> SoundModAssetTypeActions;

public:
	virtual void StartupModule() override
	{
		// Register the asset type
		SoundModAssetTypeActions = MakeShareable(new FAssetTypeActions_SoundMod);
		FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get().RegisterAssetTypeActions(SoundModAssetTypeActions.ToSharedRef());
	}

	virtual void ShutdownModule() override
	{
		if (!UObjectInitialized())
		{
			return;
		}

		// Only unregister if the asset tools module is loaded.  We don't want to forcibly load it during shutdown phase.
		check(SoundModAssetTypeActions.IsValid());
		if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
		{
			FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get().UnregisterAssetTypeActions(SoundModAssetTypeActions.ToSharedRef());
		}
		SoundModAssetTypeActions.Reset();
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FSoundModImporterModule, SoundModImporter);
DEFINE_LOG_CATEGORY(LogSoundModImporter);
