// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SoundModImporter : ModuleRules
{
	public SoundModImporter(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"SlateCore",
                "Slate",
				"SoundMod",
				"EditorFramework",
				"UnrealEd",
				"ToolMenus",
			});

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetTools",
			});

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"AssetRegistry"
			});

		PublicDefinitions.AddRange(
			new string[] {
					"BUILDING_STATIC"
				}
		);

		// Link with managed Perforce wrapper assemblies
		AddEngineThirdPartyPrivateStaticDependencies(Target, "coremod");
	
	}
}
